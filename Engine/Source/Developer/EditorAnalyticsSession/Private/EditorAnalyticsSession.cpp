// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorAnalyticsSession.h"
#include "Modules/ModuleManager.h"
#include "Internationalization/Regex.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformMisc.h"
#include "HAL/FileManager.h"
#include "Misc/DateTime.h"
#include "Misc/EngineVersion.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"

IMPLEMENT_MODULE(FEditorAnalyticsSessionModule, EditorAnalyticsSession);

namespace EditorAnalyticsDefs
{
	// The storage location is used to version the different data format. This is to prevent one version of the Editor/CRC to send sessions produced by another incompatible version.
	//   Version 1_0 : Used from creation up to 4.25.0 release (included).
	//   Version 1_1 : To avoid public API changes in 4.25.1, TotalUserInactivitySeconds was repurposed to contain the SessionDuration read from FPlatformTime::Seconds() to detect cases where the user system date time is unreliable.
	//   Version 1_2 : Removed TotalUserInactivitySeconds and added SessionDuration.
	//   Version 1_3 : Added SessionTickCount, UserInteractionCount, IsCrcExeMissing, IsUserLoggingOut, MonitorExitCode and readded lost code to save/load/delete IsLowDriveSpace for 4.26.0.
	//   Version 1_4 : Added CommandLine, EngineTickCount, LastTickTimestamp, DeathTimestamp and IsDebuggerIgnored for 4.26.0.
	//   Version 1_5 : Added Stall Detector stats for 5.0.
	//   Version 1_6 : Added ProcessDiagnostics, Renamed IsDebugger as IsDebuggerPresent: This was the last version using this system before the refactor in UE 5.0.
	static const FString StoreId(TEXT("Epic Games"));
	static const FString SessionSummaryRoot(TEXT("Unreal Engine/Session Summary"));
	static const FString DeprecatedVersions[] = { // The session format used by older versions.
		SessionSummaryRoot / TEXT("1_0"),
		SessionSummaryRoot / TEXT("1_1"),
		SessionSummaryRoot / TEXT("1_2"),
		SessionSummaryRoot / TEXT("1_3"),
		SessionSummaryRoot / TEXT("1_4"),
		SessionSummaryRoot / TEXT("1_5"),
		SessionSummaryRoot / TEXT("1_6"),
	};
	static const FString SessionSummarySection = SessionSummaryRoot / TEXT("1_7"); // The current session format.
	static const FString GlobalLockName(TEXT("UE4_SessionSummary_Lock"));
	static const FString SessionListStoreKey(TEXT("SessionList"));
	static const FString TimestampStoreKey(TEXT("Timestamp"));
}

// Utilities for writing to stored values
namespace EditorAnalyticsUtils
{
	static FDateTime StringToTimestamp(FString InString)
	{
		int64 TimestampUnix;
		if (LexTryParseString(TimestampUnix, *InString))
		{
			return FDateTime::FromUnixTimestamp(TimestampUnix);
		}
		return FDateTime::MinValue();
	}

	static FString GetSessionEventLogDir()
	{
		return FString::Printf(TEXT("%sAnalytics"), FPlatformProcess::ApplicationSettingsDir());
	}

	static void DeleteLogEvents(const FString& SessionId)
	{
		// Gather the list of files
		TArray<FString> SessionEventPaths;
		IFileManager::Get().IterateDirectoryRecursively(*EditorAnalyticsUtils::GetSessionEventLogDir(), [&SessionId, &SessionEventPaths](const TCHAR* Pathname, bool bIsDir)
		{
			if (bIsDir)
			{
				if (FPaths::GetCleanFilename(Pathname).StartsWith(SessionId))
				{
					SessionEventPaths.Emplace(Pathname);
				}
			}
			return true; // Continue
		});

		// Delete the session files.
		for (const FString& EventPathname : SessionEventPaths)
		{
			IFileManager::Get().DeleteDirectory(*EventPathname, /*RequiredExist*/false, /*Tree*/false);
		}
	}

	static TArray<FString> GetSessionList()
	{
		FString SessionListString;
		FPlatformMisc::GetStoredValue(EditorAnalyticsDefs::StoreId, EditorAnalyticsDefs::SessionSummarySection, EditorAnalyticsDefs::SessionListStoreKey, SessionListString);

		TArray<FString> SessionIDs;
		SessionListString.ParseIntoArray(SessionIDs, TEXT(","));

		return MoveTemp(SessionIDs);
	}
}

void CleanupDeprecatedAnalyticSessions(const FTimespan& MaxAge)
{
	FSystemWideCriticalSection SysWideLock(EditorAnalyticsDefs::GlobalLockName, FTimespan::Zero());
	if (!SysWideLock.IsValid())
	{
		return; // Failed to lock, don't bother, this is just for cleaning old deprecated stuff, will do next time.
	}

	// Helper function to scan and clear sessions stored in sections corresponding to older versions.
	auto CleanupVersionedSection = [&MaxAge](const FString& SectionVersion)
	{
		// Try to retreive the session list corresponding the specified session format.
		FString SessionListString;
		FPlatformMisc::GetStoredValue(EditorAnalyticsDefs::StoreId, SectionVersion, EditorAnalyticsDefs::SessionListStoreKey, SessionListString);

		if (!SessionListString.IsEmpty())
		{
			TArray<FString> SessionIDs;
			SessionListString.ParseIntoArray(SessionIDs, TEXT(","));

			for (const FString& SessionID : SessionIDs)
			{
				// All versions had a 'Timestamp' field. If it is not found, the session was partially deleted and should be cleaned up.
				FString SessionSectionName = SectionVersion / SessionID;
				FString TimestampStr;
				if (FPlatformMisc::GetStoredValue(EditorAnalyticsDefs::StoreId, SessionSectionName, EditorAnalyticsDefs::TimestampStoreKey, TimestampStr))
				{
					const FTimespan SessionAge = FDateTime::UtcNow() - EditorAnalyticsUtils::StringToTimestamp(TimestampStr);
					if (SessionAge < MaxAge)
					{
						// Don't delete the section yet, it contains a session young enough that could be sent if the user launch the Editor corresponding to this session format again.
						return;
					}

					// Clean up the log events (if any) left-over by this session.
					EditorAnalyticsUtils::DeleteLogEvents(SessionID);
				}
			}
		}

		// Nothing in the section is worth keeping, delete it entirely.
		FPlatformMisc::DeleteStoredSection(EditorAnalyticsDefs::StoreId, SectionVersion);
	};

	if (IFileManager::Get().DirectoryExists(*EditorAnalyticsUtils::GetSessionEventLogDir()))
	{
		int32 FileCount = 0;
		// Find the 'log events' directory that could be left over from previous executions.
		FRegexPattern Pattern(TEXT(R"((^[a-fA-F0-9-]+)_([0-9]+)_([0-9]+)_([0-9]+)_([0-9]+)_([0-9]+)_([0-9]+))")); // Need help with regex? Try https://regex101.com/
		IFileManager::Get().IterateDirectory(*EditorAnalyticsUtils::GetSessionEventLogDir(), [&Pattern, &MaxAge, &FileCount](const TCHAR* Pathname, bool bIsDir)
		{
			++FileCount;
			if (bIsDir) // Log events were encoded in the directory name.
			{
				FRegexMatcher Matcher(Pattern, FPaths::GetCleanFilename(Pathname));
				if (Matcher.FindNext())
				{
					FFileStatData DirStats = IFileManager::Get().GetStatData(Pathname);
					if ((FDateTime::UtcNow() - DirStats.CreationTime).GetTotalSeconds() >= MaxAge.GetTotalSeconds())
					{
						if (IFileManager::Get().DeleteDirectory(Pathname))
						{
							--FileCount;
						}
					}
				}
			}
			return true; // Continue.
		});

		if (FileCount == 0)
		{
			IFileManager::Get().DeleteDirectory(*EditorAnalyticsUtils::GetSessionEventLogDir());
		}
	}

	// Delete older and incompatible sections unless it contains a valid session young enough that would be picked up
	// if an older Editor with compatible format was launched again.
	for (int i = 0; i < UE_ARRAY_COUNT(EditorAnalyticsDefs::DeprecatedVersions); ++i)
	{
		CleanupVersionedSection(EditorAnalyticsDefs::DeprecatedVersions[i]);
	}
}
