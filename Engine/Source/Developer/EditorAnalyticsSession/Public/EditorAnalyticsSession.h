// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/Timespan.h"
#include "Modules/ModuleInterface.h"

struct UE_DEPRECATED(5.0, "Deprecated. This class was replaced by FEngineAnalyticsSessionSummary/FEditorAnalyticsSessionSummary") FEditorAnalyticsSession;

struct FEditorAnalyticsSession
{
};

/** This function removes expired sessions that were created by the deprecated system. */
EDITORANALYTICSSESSION_API void CleanupDeprecatedAnalyticSessions(const FTimespan& MaxAge);


class FEditorAnalyticsSessionModule : public IModuleInterface
{
};
