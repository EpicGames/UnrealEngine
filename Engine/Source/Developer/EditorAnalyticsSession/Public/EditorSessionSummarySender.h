// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"

class UE_DEPRECATED(5.0, "Deprecated. This class was replaced by FAnalyticsSessionSummarySender") FEditorSessionSummarySender;
class FEditorSessionSummarySender
{
};
